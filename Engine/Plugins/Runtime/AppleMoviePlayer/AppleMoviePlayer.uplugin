{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Apple Movie Player",
	"Description": "Apple Platform Movie Player using AVPlayer library",
	"Category": "Media Players",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AppleMoviePlayer",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [
				"IOS",
				"TVOS",
				"Mac"
			]
		}
	],
	"IsExperimentalVersion": false
}