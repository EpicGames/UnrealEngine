// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AppleMoviePlayer : ModuleRules
	{
        public AppleMoviePlayer(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
				    "CoreUObject",
				    "Engine",
                    "MoviePlayer",
                    "RenderCore",
                    "RHI",
                    "Slate",
					"ApplicationCore",
					"MetalRHI"
				}
			);

			AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
        
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"SlateCore",
				}
			);

			if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				PublicFrameworks.AddRange(
					new string[] {
						"QuartzCore"
					}
				);
			}
		}
	}
}
