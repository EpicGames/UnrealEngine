// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleMoviePlayer.h"
#include "Modules/ModuleInterface.h"
#include "MoviePlayer.h"
#include "AppleMovieStreamer.h"

#include "Misc/CoreDelegates.h"

TSharedPtr<FAVPlayerMovieStreamer, ESPMode::ThreadSafe> AppleMovieStreamer;

class FAppleMoviePlayerModule : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		FAVPlayerMovieStreamer *Streamer = new FAVPlayerMovieStreamer;
		AppleMovieStreamer = MakeShareable(Streamer);

        FCoreDelegates::RegisterMovieStreamerDelegate.Broadcast(AppleMovieStreamer);
	}

	virtual void ShutdownModule() override
	{
        if (AppleMovieStreamer.IsValid())
        {
            FCoreDelegates::UnRegisterMovieStreamerDelegate.Broadcast(AppleMovieStreamer);
            AppleMovieStreamer.Reset();
        }
	}
};

IMPLEMENT_MODULE( FAppleMoviePlayerModule, AppleMoviePlayer )
