// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleMovieStreamer.h"

#include "Rendering/RenderingCommon.h"
#include "RenderingThread.h"
#include "Slate/SlateTextures.h"
#include "MoviePlayer.h"
#include "Misc/CommandLine.h"
#include "Misc/ScopeLock.h"
#include "Misc/Paths.h"
#include "Misc/CoreDelegates.h"
#include "Containers/ResourceArray.h"

#include "IMetalDynamicRHI.h"
#include "Apple/AppleStringUtils.h"

DEFINE_LOG_CATEGORY_STATIC(LogMoviePlayer, Log, All);

#define MOVIE_FILE_EXTENSION @"mp4"
#define TIMESCALE 1000

#if PLATFORM_IOS
extern bool GIsSuspended;
#endif

static FString ConvertToNativePath(const FString& Filename, bool bForWrite)
{
	FString Result = Filename;
#if !PLATFORM_MAC
	if (Result.Contains(TEXT("/OnDemandResources/")))
	{
		return Result;
	}
	
	Result.ReplaceInline(TEXT("../"), TEXT(""));
	Result.ReplaceInline(TEXT(".."), TEXT(""));
	Result.ReplaceInline(FPlatformProcess::BaseDir(), TEXT(""));
	
	if(bForWrite)
	{
#if FILESHARING_ENABLED
		static FString WritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSLibraryDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
#else
		static FString WritePathBase = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
#endif
		return WritePathBase + Result;
	}
	else
	{
		// if filehostip exists in the command line, cook on the fly read path should be used
		FString Value;
		// Cache this value as the command line doesn't change...
		static bool bHasHostIP = FParse::Value(FCommandLine::Get(), TEXT("filehostip"), Value) || FParse::Value(FCommandLine::Get(), TEXT("streaminghostip"), Value);
		static bool bIsIterative = FParse::Value(FCommandLine::Get(), TEXT("iterative"), Value);
		if (bHasHostIP)
		{
			static FString ReadPathBase = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
			return ReadPathBase + Result;
		}
		else if (bIsIterative)
		{
			static FString ReadPathBase = FString([NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0]) + TEXT("/");
			return ReadPathBase + Result.ToLower();
		}
		else
		{
			static FString ReadPathBase = FString([[NSBundle mainBundle] bundlePath]) + TEXT("/cookeddata/");
			return ReadPathBase + Result.ToLower();
		}
	}
#else
	return Result;
#endif
}

//
// Methods
//

FAVPlayerMovieStreamer::FAVPlayerMovieStreamer() :
AudioPlayer         ( NULL ),
AVMovie             ( NULL ),
AVReader            ( NULL ),
AVVideoOutput       ( NULL ),
AVVideoTrack        ( NULL ),
LatestSamples       ( NULL ),
VideoRate           ( 0.0f ),
SyncStatus          ( Default ),
StartTime           ( 0.0 ),
Cursor              ( 0.0 ),
bVideoTracksLoaded  ( false ),
bWasActive          ( false ),
bIsMovieInterrupted	( false ),
ResumeTime			( kCMTimeZero ),
MetalTextureCache	( NULL )
{
	UE_LOGF(LogMoviePlayer, Log, "FAVMoviePlayer ctor...");

    MovieViewport = MakeShareable(new FMovieViewport());
}

FAVPlayerMovieStreamer::~FAVPlayerMovieStreamer()
{
#if !PLATFORM_MAC // Crash on quit - the plugin won't be destroyed until after everything has been de-inited.
    UE_LOGF(LogMoviePlayer, Log, "FAVMoviePlayer dtor...");
#endif

    // Clean up any remaining resources
    Cleanup();
}

void FAVPlayerMovieStreamer::ForceCompletion()
{
    // Stop the player and make sure it doesn't attempt to start the next movie
    MovieQueue.Empty();

    if( bVideoTracksLoaded && AVReader != nil )
    {
        [AVReader cancelReading];
    }
    if( AudioPlayer != nil )
    {
        [AudioPlayer stop];
    }

    // Teardown playback
    TeardownPlayback();
}

bool FAVPlayerMovieStreamer::Init(const TArray<FString>& MoviePaths, TEnumAsByte<EMoviePlaybackType> inPlaybackType)
{
	if(MetalTextureCache == NULL && FApp::CanEverRender())
	{
        id<MTLDevice> Device = (__bridge id<MTLDevice>)GetIMetalDynamicRHI()->RHIGetDevice();
		check(Device != nil);

		CVReturn Return = CVMetalTextureCacheCreate(kCFAllocatorDefault, nullptr, Device, nullptr, &MetalTextureCache);
		check(Return == kCVReturnSuccess);
		check(MetalTextureCache != NULL);
	}
	
	if(MetalTextureCache == NULL)
	{
		return false;
	}

	// 
	// Initializes the streamer for audio and video playback of the given path(s).
	// NOTE: If multiple paths are provided, it is expect that they be played back seamlessly.
	UE_LOGF(LogMoviePlayer, Log, "FAVMoviePlayer init. Path count = %d...", MoviePaths.Num());

	// Add the given paths to the movie queue
	MovieQueue.Append(MoviePaths);

	// Play the next movie in the queue
	return StartNextMovie();
}

bool FAVPlayerMovieStreamer::Tick(FRHICommandListBase& RHICmdList, float DeltaTime)
{
	FScopeLock LockVideoTracksLoading(&VideoTracksLoadingLock);

	if (bIsMovieInterrupted)
	{
		return false;
	}

    if( bVideoTracksLoaded )
    {
        // Remember that we were active. Used to edge detect active/not-active transitions
        bWasActive = true;

		CheckForNextFrameAndCopy(RHICmdList);

        check(AVReader != nil);
        AVAssetReaderStatus Status = [AVReader status];
        switch( Status )
        {
        case AVAssetReaderStatusReading:
            // Good...
            break;
        case AVAssetReaderStatusCompleted:
            // Mark video so that we can move on.
            bVideoTracksLoaded = false;
            break;
        case AVAssetReaderStatusFailed:
            UE_LOGF(LogMoviePlayer, Error, "Movie reader entered Failure status." );
			bVideoTracksLoaded = false;
            break;
        case AVAssetReaderStatusCancelled:
            UE_LOGF(LogMoviePlayer, Error, "Movie reader entered Cancelled status." );
			bVideoTracksLoaded = false;
            break;
        default:
            UE_LOGF(LogMoviePlayer, Error, "Movie reader encountered unknown error." );
			bVideoTracksLoaded = false;
            break;
        }
    }
    else
    {
        // We aren't active any longer, so stop the player
        if (bWasActive)
        {
			// Not active - has to try to reload tracks to become active again - keeps 2nd/3rd..etc.. videos same behaviour as 1st
			bWasActive = false;
			
            // The previous playback is complete, so shutdown.
            // NOTE: The texture resources are not freed here.
            TeardownPlayback();

			UE_LOGF(LogMoviePlayer, Verbose, "%d movie(s) left to play.", MovieQueue.Num());
            // If there are still movies to play, start the next movie
            if (MovieQueue.Num() > 0)
            {
                StartNextMovie();

                // Still playing a movie, so return that we aren't done yet
                return false;
            }
            else
            {
                // Done.
                return true;
            }
        }
		else
		{
			// No movie object and nothing left in queue - it's done - probably an error case - movie does not exist or failed to load
			// Otherwise waiting for load operation
			return AVMovie == nil && MovieQueue.Num() == 0;
		}
    }

    // Not completed.
    return false;	
}

TSharedPtr<class ISlateViewport> FAVPlayerMovieStreamer::GetViewportInterface()
{
	return MovieViewport;
}

float FAVPlayerMovieStreamer::GetAspectRatio() const
{
	return (float)MovieViewport->GetSize().X / (float)MovieViewport->GetSize().Y;
}

void FAVPlayerMovieStreamer::Cleanup()
{
	// Reset variables
	bWasActive = false;
	SyncStatus = Default;

	if( LatestSamples != NULL )
	{
		CFRelease( LatestSamples );
		LatestSamples = NULL;
	}
	
	MovieViewport->SetTexture(NULL);
	
	if(SlateTexture.IsValid() && SlateTexture->IsValid())
	{
		TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> SlateTextureCleanUp = SlateTexture;
		ENQUEUE_RENDER_COMMAND(FAVPlayerMovieStreamerCleanUp)
		(
			[SlateTextureCleanUp](FRHICommandListImmediate& RHICmdList)
			{
				SlateTextureCleanUp->ReleaseRHI();
			}
		);
		
		SlateTexture.Reset();
	}

	if (MetalTextureCache != nullptr)
	{
		CFRelease(MetalTextureCache);
		MetalTextureCache = nullptr;
	}
}

bool FAVPlayerMovieStreamer::StartNextMovie()
{
	UE_LOGF(LogMoviePlayer, Verbose, "Starting next movie...." );
	ResumeTime = kCMTimeZero;

	// Reset flag to indicate the movie may have started, but isn't playing yet.
	bVideoTracksLoaded = false;

    if (MovieQueue.Num() > 0)
    {
        if( AVMovie != NULL )
        {
            // Can't setup playback when already set up.
            UE_LOGF(LogMoviePlayer, Error, "can't setup FAVPlayerMovieStreamer because it is already set up");
            return false;
        }

        NSURL* nsURL = nil;
		FString MoviePath = FPaths::ProjectContentDir() + TEXT("Movies/") + MovieQueue[0] + TEXT(".") + FString(MOVIE_FILE_EXTENSION);
		if (FPaths::FileExists(MoviePath))
		{
			nsURL = [NSURL fileURLWithPath:ConvertToNativePath(MoviePath, false).GetNSString()];
		}
		
        if (nsURL == nil)
        {
            UE_LOGF(LogMoviePlayer, Warning, "Couldn't find movie: %ls", *MovieQueue[0]);
            MovieQueue.RemoveAt(0);
            return false;
        }
        
        NSError *error = nil;
        AudioPlayer = [[AVAudioPlayer alloc] initWithContentsOfURL:nsURL error:&error];
        if ( AudioPlayer == nil )
        {
            UE_LOGF(LogMoviePlayer, Warning, "couldn't initialize Movie player audio, bad file, or possibly just no Audio");
        }
		else
		{
			AudioPlayer.numberOfLoops = 0;
			AudioPlayer.volume = 1;
			[AudioPlayer prepareToPlay];
		}
		MovieName = MovieQueue[0];
		MovieQueue.RemoveAt(0);

		return LoadMovieAsync(MovieName);
	}
	return false;
}

bool FAVPlayerMovieStreamer::LoadMovieAsync(FString InMovieName)
{
	FScopeLock LockVideoTracksLoading(&VideoTracksLoadingLock);
	
	// Reset flag to indicate the movie may have started, but isn't playing yet.
	bVideoTracksLoaded = false;
	NSURL* nsURL = nil;
	FString MoviePath = FPaths::ProjectContentDir() + TEXT("Movies/") + InMovieName + TEXT(".") + FString(MOVIE_FILE_EXTENSION);
	if (FPaths::FileExists(MoviePath))
	{
		nsURL = [NSURL fileURLWithPath : ConvertToNativePath(MoviePath, false).GetNSString()];
	}
	
	if (nsURL == nil)
	{
		return false;
	}

	// Load the Movie with the appropriate URL.
    AVMovie = [[AVURLAsset alloc] initWithURL:nsURL options:nil];

    // Obtain the tracks asynchronously.
    NSArray* nsTrackKeys = @[@"tracks"];
    [AVMovie loadValuesAsynchronouslyForKeys:nsTrackKeys completionHandler:^()
    {
    	FScopeLock AynscLockVideoTracksLoading(&VideoTracksLoadingLock);

        // Once loaded, initialize our reader object to start pulling frames
        bVideoTracksLoaded = FinishLoadingTracks();
        
#if PLATFORM_IOS
		bIsMovieInterrupted = GIsSuspended;
#endif
    }];
	
    return true;
}

bool FAVPlayerMovieStreamer::FinishLoadingTracks()
{
    NSError *nsError = nil;
    AVKeyValueStatus TrackStatus = [AVMovie statusOfValueForKey:@"tracks" error:&nsError];

    bool bLoadedAndReading = false;
    switch( TrackStatus )
    {
    case AVKeyValueStatusLoaded:
        {
            // Tracks loaded correctly!

            // Create a reader to actually process the tracks.
            AVReader = [[AVAssetReader alloc] initWithAsset: AVMovie error: &nsError];
            check(AVReader != nil);

            // The media may have multiple tracks (like audio), but we'll just grab the first Video track.
            NSArray* nsVideoTracks = [AVMovie tracksWithMediaType:AVMediaTypeVideo];
            if( [nsVideoTracks count] == 0 )
            {
                UE_LOGF(LogMoviePlayer, Error, "Movie contains no Video tracks." );
                return false;
            }
            else
            {
                // Save the track for later.
                AVVideoTrack = [nsVideoTracks objectAtIndex:0];
				
				CGSize naturalSize = [AVVideoTrack naturalSize];
				if((int(naturalSize.width) % 16) != 0)
				{
					UE_LOGF(LogMoviePlayer, Error, "Movie width must be a multiple of 16 pixels." );
					return false;
				}
				
                // Initialize our video output to match the format of the texture we'll create later.
                NSMutableDictionary* OutputSettings = [NSMutableDictionary dictionary];
				
                [OutputSettings setObject:[NSNumber numberWithInt:kCVPixelFormatType_32BGRA] forKey:(NSString*)kCVPixelBufferPixelFormatTypeKey];
                [OutputSettings setObject:[NSDictionary dictionary] forKey:(id)kCVPixelBufferIOSurfacePropertiesKey];
				[OutputSettings setObject:[NSNumber numberWithBool:YES] forKey:(NSString*)kCVPixelBufferMetalCompatibilityKey];
				[OutputSettings setObject:[NSNumber numberWithInteger:1] forKey:(NSString*)kCVPixelBufferBytesPerRowAlignmentKey];
				
                AVVideoOutput = [[AVAssetReaderTrackOutput alloc] initWithTrack:AVVideoTrack outputSettings:OutputSettings];
                AVVideoOutput.alwaysCopiesSampleData = NO;

                // Assign the track to the reader.
                [AVReader addOutput:AVVideoOutput];

                // Begin reading!
				[AVReader setTimeRange : CMTimeRangeMake(ResumeTime, kCMTimePositiveInfinity)];

                if( ![AVReader startReading] )
                {
                    UE_LOGF(LogMoviePlayer, Error, "AVReader 'startReading' returned failure.");
                    return false;
                }

                // Save the rate of playback.
                check( AVVideoTrack.nominalFrameRate );
                VideoRate = 1.0f / AVVideoTrack.nominalFrameRate;

                // Reset the starting time.
                StartTime = 0.0;

                // Good to go.
                bLoadedAndReading = true;
            }

        } break;
    case AVKeyValueStatusFailed:
        {
            UE_LOGF(LogMoviePlayer, Error, "Failed to load Tracks for Movie." );
        } break;
    case AVKeyValueStatusCancelled:
        {
            UE_LOGF(LogMoviePlayer, Error, "Cancelled loading Tracks for Movie." );
        } break;
    default:
        {
            UE_LOGF(LogMoviePlayer, Error, "Unknown error loading Tracks for Movie." );
        } break;
    }

    return bLoadedAndReading;
}

bool FAVPlayerMovieStreamer::CheckForNextFrameAndCopy(FRHICommandListBase& RHICmdList)
{
    check( bVideoTracksLoaded );

    bool bHasNewFrame = false;

    // We need to synchronize the video playback with the audio.
    // If the video frame is within tolerance (Ready), update the Texture.
    // If the video is Behind, throw it away and get the next one until we catch up with the ref time.
    // If the video is Ahead, update the Texture but don't retrieve more frames until time catches up.

    if(StartTime == 0.0)
    {
    	// Now kick everything going at the same time
		StartTime = CACurrentMediaTime() - CMTimeGetSeconds(ResumeTime);
		
		if(AudioPlayer != nil && !AudioPlayer.isPlaying)
		{
			[AudioPlayer play];
		}
    }
	
    while( SyncStatus != Ready )
    {
        if( SyncStatus != Ahead )
        {
            LatestSamples = [AVVideoOutput copyNextSampleBuffer];
        }
        if( LatestSamples == NULL )
        {
            // Failed to get next sample buffer.
            break;
        }

        // Get the time stamp of the video frame
        CMTime FrameTimeStamp = CMSampleBufferGetPresentationTimeStamp(LatestSamples);

        // Get the time since playback began
        Cursor = CACurrentMediaTime() - StartTime;

        // Compute delta of video frame and current playback times
        double Delta = Cursor - CMTimeGetSeconds(FrameTimeStamp);

        if( Delta < 0.0 )
        {
            Delta *= - 1.0;
            SyncStatus = Ahead;
        }
        else
        {
            SyncStatus = Behind;
        }

        if( Delta < VideoRate )
        {
             // Video in sync with audio
            SyncStatus = Ready;
        }
        else if(SyncStatus == Ahead)
        {
            // Video ahead of audio: stay in Ahead state, exit loop
            break;
        }
        else
        {
            // Video behind audio (Behind): stay in loop
            CFRelease(LatestSamples);
            LatestSamples = NULL;
        }
    }
    

    // Do we have a new frame to process?
    if( SyncStatus == Ready )
    {
        check(LatestSamples != NULL);
		check(MetalTextureCache != NULL);

        CVImageBufferRef pPixelBuffer = CMSampleBufferGetImageBuffer(LatestSamples);
		
		CGSize Size = CVImageBufferGetEncodedSize(pPixelBuffer);
		
		uint32 Width  = (uint32)Size.width;
		uint32 Height = (uint32)Size.height;
	
		CVMetalTextureRef TextureCacheRef = NULL;
		CVReturn Result = CVMetalTextureCacheCreateTextureFromImage(kCFAllocatorDefault, MetalTextureCache, pPixelBuffer, nullptr, MTLPixelFormatBGRA8Unorm, Width, Height, 0, &TextureCacheRef);
		check(Result == kCVReturnSuccess);
		check(TextureCacheRef);

		IMetalDynamicRHI* MetalDynamicRHI = GetIMetalDynamicRHI();
					
		FTextureRHIRef MediaWrappedTexture = MetalDynamicRHI->RHICreateTexture2DFromCVMetalTexture(PF_B8G8R8A8, ETextureCreateFlags::NoTiling | ETextureCreateFlags::ShaderResource, FClearValueBinding::White, TextureCacheRef);
		CFRelease(TextureCacheRef);
		
		if(SlateTexture.IsValid())
		{
			if(SlateTexture->IsValid())
			{
				SlateTexture->ReleaseRHI();
			}
			SlateTexture->SetRHIRef(MediaWrappedTexture, Width, Height);
		}
		else
		{
			SlateTexture = MakeShareable(new FSlateTexture2DRHIRef(MediaWrappedTexture, Width, Height));
			MovieViewport->SetTexture(SlateTexture);
		}

        CFRelease(LatestSamples);
        LatestSamples = NULL;

        bHasNewFrame = true;
    }
    if( SyncStatus != Ahead )
    {
        // Reset status;
        SyncStatus = Default;
    }

    return bHasNewFrame;
}

void FAVPlayerMovieStreamer::TeardownPlayback()
{
    if( LatestSamples != NULL )
    {
        CFRelease( LatestSamples );
        LatestSamples = NULL;
    }
    
	ReleaseMovie();
	
    if ( AudioPlayer != nil )
    {
        AudioPlayer = nil;
    }
}

void FAVPlayerMovieStreamer::ReleaseMovie()
{
	if (AVMovie != nil)
	{
		AVMovie = nil;
	}
	// NS Object release handled by external AutoReleasePool
	if (AVVideoOutput != nil)
	{
		AVVideoOutput = nil;
	}
	if (AVVideoTrack != nil)
	{
		AVVideoTrack = nil;
	}
	if (AVReader != nil)
	{
		AVReader = nil;
	}

}

FString FAVPlayerMovieStreamer::GetMovieName()
{
	return MovieName;
}

bool FAVPlayerMovieStreamer::IsLastMovieInPlaylist()
{
	return MovieQueue.Num() <= 1;
}


/* Interrupts */
void FAVPlayerMovieStreamer::Suspend()
{
	if (bIsMovieInterrupted)
	{
		//already paused
		return;
	}
	bIsMovieInterrupted = true;

	FScopeLock LockVideoTracksLoading(&VideoTracksLoadingLock);

	if (bVideoTracksLoaded && AVReader != nil)
	{
		ResumeTime = CMTimeMake(Cursor * TIMESCALE, TIMESCALE);

		[AudioPlayer pause];

		[AVReader cancelReading];

	}
	ReleaseMovie();
}

void FAVPlayerMovieStreamer::Resume()
{
	if (!bIsMovieInterrupted)
	{
		//already resumed
		return;
	}
	
	LoadMovieAsync(MovieName);
}
