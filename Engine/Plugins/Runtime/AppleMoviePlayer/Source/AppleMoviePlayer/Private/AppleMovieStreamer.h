// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MoviePlayer.h"

#include "Slate/SlateTextures.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <AVFoundation/AVFoundation.h>


// The actual streamer class
class FAVPlayerMovieStreamer : public IMovieStreamer
{
public:
	FAVPlayerMovieStreamer();
	virtual ~FAVPlayerMovieStreamer();

	virtual bool Init(const TArray<FString>& MoviePaths, TEnumAsByte<EMoviePlaybackType> inPlaybackType) override;
	virtual void ForceCompletion() override;
	virtual bool Tick(FRHICommandListBase& RHICmdList, float DeltaTime) override;
	virtual TSharedPtr<class ISlateViewport> GetViewportInterface() override;
	virtual float GetAspectRatio() const override;
	virtual void Cleanup() override;

	virtual FString GetMovieName() override;
	virtual bool IsLastMovieInPlaylist() override;
	virtual void Suspend() override;
	virtual void Resume() override;

	FOnCurrentMovieClipFinished OnCurrentMovieClipFinishedDelegate;
	virtual FOnCurrentMovieClipFinished& OnCurrentMovieClipFinished() override { return OnCurrentMovieClipFinishedDelegate; }

private:

    enum ESyncStatus
    {
        Default,    // Starting state
        Ahead,      // Frame is ahead of playback cursor. 
        Behind,     // Frame is behind playback cursor. 
        Ready,      // Frame is within tolerance of playback cursor.
    };

	/** Texture and viewport data for displaying to Slate */
	TSharedPtr<FMovieViewport> MovieViewport;

    TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> SlateTexture;

    // The list of pending movies
    TArray<FString>		        MovieQueue;

    // Current Movie
    AVAudioPlayer*              AudioPlayer;
    AVURLAsset*                 AVMovie;
    AVAssetReader*              AVReader;
    AVAssetReaderTrackOutput*   AVVideoOutput;
    AVAssetTrack*               AVVideoTrack;
    CMSampleBufferRef           LatestSamples;

	FString MovieName;

    // AV Synchronization
    float                       VideoRate;
    int                         SyncStatus;
    double                      StartTime;
    double                      Cursor;

    bool                        bVideoTracksLoaded;
    bool                        bWasActive;

	bool						bIsMovieInterrupted;
	CMTime						ResumeTime;

	FCriticalSection			VideoTracksLoadingLock;
	
	CVMetalTextureCacheRef 		MetalTextureCache;

private:

    /**
	 * Sets up and starts playback on the next movie in the queue
	 */
	bool StartNextMovie();

    bool FinishLoadingTracks();

    void TeardownPlayback();

    bool CheckForNextFrameAndCopy(FRHICommandListBase& RHICmdList);

	void ReleaseMovie();

	bool LoadMovieAsync(FString MovieName);
};
